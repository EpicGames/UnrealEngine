// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/SharedString.h"
#include "HAL/FileManager.h"
#include "Misc/CommandLine.h"
#include "Containers/IndirectArray.h"
#include "Stats/Stats.h"
#include "Async/AsyncWork.h"
#include "Misc/ConfigCacheIni.h"
#include "ImageCore.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/ITextureFormat.h"
#include "Interfaces/ITextureFormatModule.h"
#include "TextureCompressorModule.h"
#include "PixelFormat.h"
#include "Async/ParallelFor.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryWriter.h"
#include "TextureBuildFunction.h"
#include "DerivedDataBuildFunctionFactory.h"
#include "Misc/WildcardString.h"

#include "ispc_texcomp.h"

DEFINE_LOG_CATEGORY_STATIC(LogTextureFormatIntelISPCTexComp, Log, All);

class FIntelISPCTexCompTextureBuildFunction final : public FTextureBuildFunction
{
	const UE::FUtf8SharedString& GetName() const final
	{
		static const UE::FUtf8SharedString Name(UTF8TEXTVIEW("IntelISPCTexCompTexture"));
		return Name;
	}

	void GetVersion(UE::DerivedData::FBuildVersionBuilder& Builder, ITextureFormat*& OutTextureFormatVersioning) const final
	{
		static FGuid Version(TEXT("19d413ad-f529-4687-902a-3b71919cfd72"));
		Builder << Version;
		OutTextureFormatVersioning = FModuleManager::GetModuleChecked<ITextureFormatModule>(TEXT("TextureFormatIntelISPCTexComp")).GetTextureFormat();
	}
};

// increment this if you change anything that will affect compression in this file
#define BASE_ISPC_DX11_FORMAT_VERSION 9

// For debugging intermediate image results by saving them out as files.
#define DEBUG_SAVE_INTERMEDIATE_IMAGES 0

/**
 * Macro trickery for supported format names.
 */
#define ENUM_SUPPORTED_FORMATS(op) \
	op(BC6H) \
	op(BC7)

	#define DECL_FORMAT_NAME(FormatName) static FName GTextureFormatName##FormatName = FName(TEXT(#FormatName));
	ENUM_SUPPORTED_FORMATS(DECL_FORMAT_NAME);
	#undef DECL_FORMAT_NAME

	#define DECL_FORMAT_NAME_ENTRY(FormatName) GTextureFormatName##FormatName ,
	static FName GSupportedTextureFormatNames[] =
	{
		ENUM_SUPPORTED_FORMATS(DECL_FORMAT_NAME_ENTRY)
	};
	#undef DECL_FORMAT_NAME_ENTRY
#undef ENUM_SUPPORTED_FORMATS

/**
 * note : ASTC formats are NOT in the GSupportedTextureFormatNames
 *  so we are not registered to support them
 *  they will call into TextureFormatASTC and then optionally redirect to here
 * this just defines the FNames
 */
#define ENUM_ASTC_FORMATS(op) \
	op(ASTC_RGB) \
	op(ASTC_RGBA) \
	op(ASTC_RGBAuto) \
	op(ASTC_RGBA_HQ) \
	op(ASTC_RGB_HDR) \
	op(ASTC_NormalAG) \
	op(ASTC_NormalRG)

	#define DECL_FORMAT_NAME(FormatName) static FName GTextureFormatName##FormatName = FName(TEXT(#FormatName));
	ENUM_ASTC_FORMATS(DECL_FORMAT_NAME);
	#undef DECL_FORMAT_NAME
#undef ENUM_ASTC_FORMATS

// BC6H, BC7, ASTC all have 16-byte block size
#define BLOCK_SIZE_IN_BYTES 16


#if DEBUG_SAVE_INTERMEDIATE_IMAGES
namespace
{

	// Bitmap compression types.
enum EBitmapCompression
{
	BCBI_RGB = 0,
	BCBI_RLE8 = 1,
	BCBI_RLE4 = 2,
	BCBI_BITFIELDS = 3,
};

// .BMP file header.
#pragma pack(push,1)
struct FBitmapFileHeader
{
	uint16 bfType;
	uint32 bfSize;
	uint16 bfReserved1;
	uint16 bfReserved2;
	uint32 bfOffBits;
	friend FArchive& operator<<(FArchive& Ar, FBitmapFileHeader& H)
	{
		Ar << H.bfType << H.bfSize << H.bfReserved1 << H.bfReserved2 << H.bfOffBits;
		return Ar;
	}
};
#pragma pack(pop)

// .BMP subheader.
#pragma pack(push,1)
struct FBitmapInfoHeader
{
	uint32 biSize;
	uint32 biWidth;
	int32 biHeight;
	uint16 biPlanes;
	uint16 biBitCount;
	uint32 biCompression;
	uint32 biSizeImage;
	uint32 biXPelsPerMeter;
	uint32 biYPelsPerMeter;
	uint32 biClrUsed;
	uint32 biClrImportant;
	friend FArchive& operator<<(FArchive& Ar, FBitmapInfoHeader& H)
	{
		Ar << H.biSize << H.biWidth << H.biHeight;
		Ar << H.biPlanes << H.biBitCount;
		Ar << H.biCompression << H.biSizeImage;
		Ar << H.biXPelsPerMeter << H.biYPelsPerMeter;
		Ar << H.biClrUsed << H.biClrImportant;
		return Ar;
	}
};
#pragma pack(pop)


static void SaveImageAsBMP( FArchive& Ar, const uint8* RawData, int SourceBytesPerPixel, int SizeX, int SizeY )
{
	FBitmapFileHeader bmf;
	FBitmapInfoHeader bmh;

	// File header.
	bmf.bfType = 'B' + (256 * (int32)'M');
	bmf.bfReserved1 = 0;
	bmf.bfReserved2 = 0;
	int32 biSizeImage = SizeX * SizeY * 3;
	bmf.bfOffBits = sizeof(FBitmapFileHeader) + sizeof(FBitmapInfoHeader);
	bmh.biBitCount = 24;

	bmf.bfSize = bmf.bfOffBits + biSizeImage;
	Ar << bmf;

	// Info header.
	bmh.biSize = sizeof(FBitmapInfoHeader);
	bmh.biWidth = SizeX;
	bmh.biHeight = SizeY;
	bmh.biPlanes = 1;
	bmh.biCompression = BCBI_RGB;
	bmh.biSizeImage = biSizeImage;
	bmh.biXPelsPerMeter = 0;
	bmh.biYPelsPerMeter = 0;
	bmh.biClrUsed = 0;
	bmh.biClrImportant = 0;
	Ar << bmh;

	bool bIsRGBA16 = (SourceBytesPerPixel == 8);

	//NOTE: Each row must be 4-byte aligned in a BMP.
	int PaddingX = Align(SizeX * 3, 4) - SizeX * 3;

	// Upside-down scanlines.
	for (int32 i = SizeY - 1; i >= 0; i--)
	{
		const uint8* ScreenPtr = &RawData[i*SizeX*SourceBytesPerPixel];
		for (int32 j = SizeX; j > 0; j--)
		{
			uint8 R, G, B;
			if (bIsRGBA16)
			{
				R = ScreenPtr[1];
				G = ScreenPtr[3];
				B = ScreenPtr[5];
				ScreenPtr += 8;
			}
			else
			{
				R = ScreenPtr[0];
				G = ScreenPtr[1];
				B = ScreenPtr[2];
				ScreenPtr += 4;
			}
			Ar << R;
			Ar << G;
			Ar << B;
		}
		for (int32 j = 0; j < PaddingX; ++j)
		{
			int8 PadByte = 0;
			Ar << PadByte;
		}
	}
}

#define MAGIC_FILE_CONSTANT 0x5CA1AB13

// little endian
#pragma pack(push,1)
struct astc_header
{
	uint8_t magic[4];
	uint8_t blockdim_x;
	uint8_t blockdim_y;
	uint8_t blockdim_z;
	uint8_t xsize[3];
	uint8_t ysize[3];			// x-size, y-size and z-size are given in texels;
	uint8_t zsize[3];			// block count is inferred
};
#pragma pack(pop)

static void SaveImageAsASTC(FArchive& Ar, uint8* RawData, int SizeX, int SizeY, int block_width, int block_height)
{
	astc_header file_header;

	uint32_t magic = MAGIC_FILE_CONSTANT;
	FMemory::Memcpy(file_header.magic, &magic, 4);
	file_header.blockdim_x = block_width;
	file_header.blockdim_y = block_height;
	file_header.blockdim_z = 1;

	int32 xsize = SizeX;
	int32 ysize = SizeY;
	int32 zsize = 1;

	FMemory::Memcpy(file_header.xsize, &xsize, 3);
	FMemory::Memcpy(file_header.ysize, &ysize, 3);
	FMemory::Memcpy(file_header.zsize, &zsize, 3);

	Ar.Serialize(&file_header, sizeof(file_header));

	size_t height_in_blocks = (SizeY + block_height - 1) / block_height;
	size_t width_in_blocks = (SizeX + block_width - 1) / block_width;
	int stride = width_in_blocks * BLOCK_SIZE_IN_BYTES;
	Ar.Serialize(RawData, height_in_blocks * stride);
}

}; // namespace
#endif 
// DEBUG_SAVE_INTERMEDIATE_IMAGES

struct FMultithreadSettings
{
	int iScansPerTask;
	int iNumTasks;
};

template <typename EncoderSettingsType>
struct FMultithreadedCompression
{
	typedef void(*CompressFunction)(EncoderSettingsType* pEncSettings, FImage* pInImage, FCompressedImage2D* pOutImage, int yStart, int yEnd, int SliceIndex);

	static void Compress(FMultithreadSettings &MultithreadSettings, EncoderSettingsType &EncoderSettings, FImage &Image, FCompressedImage2D &OutCompressedImage, CompressFunction FunctionCallback, bool bUseTasks)
	{
		if (bUseTasks)
		{
			class FIntelCompressWorker
			{
			public:
				FIntelCompressWorker(EncoderSettingsType* pEncSettings, FImage* pInImage, FCompressedImage2D* pOutImage, int yStart, int yEnd, int SliceIndex, CompressFunction InFunctionCallback)
				:	mpEncSettings(pEncSettings)
				,	mpInImage(pInImage)
				,	mpOutImage(pOutImage)
				,	mYStart(yStart)
				,	mYEnd(yEnd)
				,	mSliceIndex(SliceIndex)
				,	mCallback(InFunctionCallback)
				{
				}

				void DoWork()
				{
					mCallback(mpEncSettings, mpInImage, mpOutImage, mYStart, mYEnd, mSliceIndex);
				}

				EncoderSettingsType*	mpEncSettings;
				FImage*					mpInImage;
				FCompressedImage2D*		mpOutImage;
				int						mYStart;
				int						mYEnd;
				int						mSliceIndex;
				CompressFunction		mCallback;
			};

			// One less task because we'll do the final + non multiple of 4 inside this task
			TArray<FIntelCompressWorker> CompressionTasks;
			const int NumStasksPerSlice = MultithreadSettings.iNumTasks + 1;
			CompressionTasks.Reserve(NumStasksPerSlice * Image.NumSlices - 1);
			for (int SliceIndex = 0; SliceIndex < Image.NumSlices; ++SliceIndex)
			{
				for (int iTask = 0; iTask < NumStasksPerSlice; ++iTask)
				{
					// Create a new task unless it's the last task in the last slice (that one will run on current thread, after these threads have been started)
					if (SliceIndex < (Image.NumSlices - 1) || iTask < (NumStasksPerSlice - 1))
					{
						CompressionTasks.Emplace(&EncoderSettings, &Image, &OutCompressedImage, iTask * MultithreadSettings.iScansPerTask, (iTask + 1) * MultithreadSettings.iScansPerTask, SliceIndex, FunctionCallback);
					}
				}
			}

			ParallelForWithPreWork(CompressionTasks.Num(), [&CompressionTasks](int32 TaskIndex)
			{
				CompressionTasks[TaskIndex].DoWork();
			},
			[&EncoderSettings, &Image, &OutCompressedImage, &MultithreadSettings, &FunctionCallback]()
			{
				FunctionCallback(&EncoderSettings, &Image, &OutCompressedImage, MultithreadSettings.iScansPerTask * MultithreadSettings.iNumTasks, Image.SizeY, Image.NumSlices - 1);
			}, EParallelForFlags::Unbalanced);
		}
		else
		{
			for (int SliceIndex = 0; SliceIndex < Image.NumSlices; ++SliceIndex)
			{
				FunctionCallback(&EncoderSettings, &Image, &OutCompressedImage, 0, Image.SizeY, SliceIndex);
			}
		}
	}
};

/**
 * BC6H Compression function
 */
static void IntelBC6HCompressScans(bc6h_enc_settings* pEncSettings, FImage* pInImage, FCompressedImage2D* pOutImage, int yStart, int yEnd, int SliceIndex)
{
	check(pInImage->Format == ERawImageFormat::RGBA16F);
	check((yStart % 4) == 0);
	// sizes are padded to multiple of 4 for ISPC by PadImageToBlockSize :
	check((pInImage->SizeX % 4) == 0);
	check((yStart >= 0) && (yStart <= pInImage->SizeY));
	check((yEnd   >= 0) && (yEnd   <= pInImage->SizeY));

	const int64 InStride = (int64)pInImage->SizeX * 8;
	const int64 OutStride = (int64)((pInImage->SizeX + 3) / 4) * BLOCK_SIZE_IN_BYTES;
	const int64 InSliceSize = (int64)pInImage->SizeY * InStride;
	const int64 OutSliceSize = (int64)((pInImage->SizeY +3) / 4) * OutStride;

	uint8* pInTexels = reinterpret_cast<uint8*>(&pInImage->RawData[0]) + InSliceSize * SliceIndex;
	uint8* pOutTexels = reinterpret_cast<uint8*>(&pOutImage->RawData[0]) + OutSliceSize * SliceIndex;

	rgba_surface insurface;
	insurface.ptr		= pInTexels + (yStart * InStride);
	insurface.width		= pInImage->SizeX;
	insurface.height	= yEnd - yStart;
	insurface.stride	= pInImage->SizeX * 8;

	pOutTexels += yStart / 4 * OutStride;
	CompressBlocksBC6H(&insurface, pOutTexels, pEncSettings);
}

/**
 * BC7 Compression function
 */
static void IntelBC7CompressScans(bc7_enc_settings* pEncSettings, FImage* pInImage, FCompressedImage2D* pOutImage, int yStart, int yEnd, int SliceIndex)
{
	check(pInImage->Format == ERawImageFormat::BGRA8);
	check((yStart % 4) == 0);
	// sizes are padded to multiple of 4 for ISPC by PadImageToBlockSize :
	check((pInImage->SizeX % 4) == 0);
	check((yStart >= 0) && (yStart <= pInImage->SizeY));
	check((yEnd   >= 0) && (yEnd   <= pInImage->SizeY));

	const int64 InStride = (int64)pInImage->SizeX * 4;
	const int64 OutStride = (int64)((pInImage->SizeX + 3) / 4) * BLOCK_SIZE_IN_BYTES;
	const int64 InSliceSize = (int64)pInImage->SizeY * InStride;
	const int64 OutSliceSize = (int64)((pInImage->SizeY + 3) / 4) * OutStride;

	uint8* pInTexels = reinterpret_cast<uint8*>(&pInImage->RawData[0]) + InSliceSize * SliceIndex;
	uint8* pOutTexels = reinterpret_cast<uint8*>(&pOutImage->RawData[0]) + OutSliceSize * SliceIndex;

	// Switch byte order for compressors input
	for ( int y=yStart; y < yEnd; ++y )
	{
		uint8* pInTexelsSwap = pInTexels + (y * InStride);
		for ( int x=0; x < pInImage->SizeX; ++x )
		{
			const uint8 r = pInTexelsSwap[0];
			pInTexelsSwap[0] = pInTexelsSwap[2];
			pInTexelsSwap[2] = r;

			pInTexelsSwap += 4;
		}
	}

	rgba_surface insurface;
	insurface.ptr		= pInTexels + (yStart * InStride);
	insurface.width		= pInImage->SizeX;
	insurface.height	= yEnd - yStart;
	insurface.stride	= pInImage->SizeX * 4;

	pOutTexels += yStart / 4 * OutStride;
	CompressBlocksBC7(&insurface, pOutTexels, pEncSettings);
}

#define MAX_QUALITY_BY_SIZE 4

static uint16 GetDefaultCompressionBySizeValue(FCbObjectView InFormatConfigOverride)
{
	// this is code duped between TextureFormatASTC and TextureFormatISPC
	if (InFormatConfigOverride)
	{
		// If we have an explicit format config, then use it directly
		FCbFieldView FieldView = InFormatConfigOverride.FindView("DefaultASTCQualityBySize");
		checkf(FieldView.HasValue(), TEXT("Missing DefaultASTCQualityBySize key from FormatConfigOverride"));
		int32 CompressionModeValue = FieldView.AsInt32();
		checkf(!FieldView.HasError(), TEXT("Failed to parse DefaultASTCQualityBySize value from FormatConfigOverride"));
		return IntCastChecked<uint16>(CompressionModeValue);
	}
	else
	{
		// default of 0 == 12x12 ?
		// BaseEngine.ini sets DefaultASTCQualityBySize to 3 == 6x6

		auto GetCompressionModeValue = []() {
			// start at default quality, then lookup in .ini file
			int32 CompressionModeValue = 0;
			GConfig->GetInt(TEXT("/Script/UnrealEd.CookerSettings"), TEXT("DefaultASTCQualityBySize"), CompressionModeValue, GEngineIni);
	
			FParse::Value(FCommandLine::Get(), TEXT("-astcqualitybysize="), CompressionModeValue);
			
			return FMath::Min<uint32>(CompressionModeValue, MAX_QUALITY_BY_SIZE);
		};

		static uint16 CompressionModeValue = IntCastChecked<uint16>(GetCompressionModeValue());

		return CompressionModeValue;
	}
}

static EPixelFormat GetASTCQualityFormat(int& BlockWidth, int& BlockHeight, const FTextureBuildSettings& BuildSettings)
{
	// code dupe between TextureFormatASTC  and TextureFormatISPC

	const FCbObjectView& InFormatConfigOverride = BuildSettings.FormatConfigOverride;
	int32 OverrideSizeValue= BuildSettings.CompressionQuality;

	bool bIsNormalMap = (BuildSettings.TextureFormatName == GTextureFormatNameASTC_NormalAG || BuildSettings.TextureFormatName == GTextureFormatNameASTC_NormalRG);
	bool bIsHQ = BuildSettings.TextureFormatName == GTextureFormatNameASTC_RGBA_HQ;
	bool bHDRFormat = BuildSettings.TextureFormatName == GTextureFormatNameASTC_RGB_HDR;
	check( ! bHDRFormat ); // HDR not supported here
	
	if ( bIsNormalMap )
	{
		BlockWidth = BlockHeight = 6;
		return PF_ASTC_6x6;
	}
	else if ( bIsHQ || BuildSettings.bVirtualStreamable )
	{
		// NOTE: different than TextureFormatASTC which configurable HQ block size
		BlockWidth = BlockHeight = 4;
		return PF_ASTC_4x4;		
	}

	// Note: ISPC only supports 8x8 and higher quality, and only one speed (fast)
	EPixelFormat Format = PF_Unknown;
	switch (OverrideSizeValue >= 0 ? OverrideSizeValue : GetDefaultCompressionBySizeValue(InFormatConfigOverride))
	{
		// NOTE: different than TextureFormatASTC which has the 10 and 12 size blocks here
		case 0:	//Format = PF_ASTC_12x12; BlockWidth = BlockHeight = 12; break;
		case 1:	//Format = PF_ASTC_10x10; BlockWidth = BlockHeight = 10; break;
		case 2:	Format = PF_ASTC_8x8; BlockWidth = BlockHeight = 8; break;
		case 3:	Format = PF_ASTC_6x6; BlockWidth = BlockHeight = 6; break;
		case 4:	Format = PF_ASTC_4x4; BlockWidth = BlockHeight = 4; break;
		default: UE_LOGF(LogTemp, Fatal, "Max quality higher than expected");
	}

	return Format;
}

struct FASTCEncoderSettings : public astc_enc_settings
{
	FName TextureFormatName;
	bool bISPCTexcompNormalizeNormals = false;

	FASTCEncoderSettings()
	{
		// ensure astc_enc_settings are initialized :
		astc_enc_settings * parent = this;
		memset(parent,0,sizeof(*parent));
	}
};


/**
 * ASTC Compression function
 */
static void IntelASTCCompressScans(FASTCEncoderSettings* pEncSettings, FImage* pInImage, FCompressedImage2D* pOutImage, int yStart, int yEnd, int SliceIndex)
{
	check(pInImage->Format == ERawImageFormat::BGRA8);
	check((yStart % pEncSettings->block_height) == 0);
	
	// sizes are padded for ISPC by PadImageToBlockSize :
	check((pInImage->SizeX % pEncSettings->block_width) == 0);
	check((yStart >= 0) && (yStart <= pInImage->SizeY));
	check((yEnd   >= 0) && (yEnd   <= pInImage->SizeY));

	const int64 InStride = (int64)pInImage->SizeX * 4;
	const int64 OutStride = (int64)FMath::DivideAndRoundUp( pInImage->SizeX , pEncSettings->block_width) * BLOCK_SIZE_IN_BYTES;
	const int64 InSliceSize = (int64)pInImage->SizeY * InStride;
	const int64 OutSliceSize = (int64)FMath::DivideAndRoundUp( pInImage->SizeY , pEncSettings->block_height) * OutStride;

	uint8* pInTexels = reinterpret_cast<uint8*>(&pInImage->RawData[0]) + InSliceSize * SliceIndex;
	uint8* pOutTexels = reinterpret_cast<uint8*>(&pOutImage->RawData[0]) + OutSliceSize * SliceIndex;

	// TextureFormatName at this point should no longer be RGBAuto
	//	it's mapped to _RGB or _RGBA concretely

	if (pEncSettings->TextureFormatName == GTextureFormatNameASTC_RGB)
	{
		// Switch byte order for compressors input (BGRA -> RGBA)
		// Force A=255
		for (int y = yStart; y < yEnd; ++y)
		{
			uint8* pInTexelsSwap = pInTexels + (y * InStride);
			for (int x = 0; x < pInImage->SizeX; ++x)
			{
				const uint8 r = pInTexelsSwap[0];
				pInTexelsSwap[0] = pInTexelsSwap[2];
				pInTexelsSwap[2] = r;
				pInTexelsSwap[3] = 255;

				pInTexelsSwap += 4;
			}
		}
	}
	else if (pEncSettings->TextureFormatName == GTextureFormatNameASTC_RGBA || pEncSettings->TextureFormatName == GTextureFormatNameASTC_RGBA_HQ)
	{
		// Switch byte order for compressors input (BGRA -> RGBA)
		for (int y = yStart; y < yEnd; ++y)
		{
			uint8* pInTexelsSwap = pInTexels + (y * InStride);
			for (int x = 0; x < pInImage->SizeX; ++x)
			{
				const uint8 r = pInTexelsSwap[0];
				pInTexelsSwap[0] = pInTexelsSwap[2];
				pInTexelsSwap[2] = r;

				pInTexelsSwap += 4;
			}
		}
	}
	else if (pEncSettings->TextureFormatName == GTextureFormatNameASTC_NormalAG)
	{
		// input BGRA -> output 0,G,0,A

		if ( pEncSettings->bISPCTexcompNormalizeNormals )
		{
			// legacy behavior:
			// Re-normalize normals before dropping components
			// this is right, but doing it here makes this TextureFormat inconsistent with all others
			for (int y = yStart; y < yEnd; ++y)
			{
				uint8* pInTexelsSwap = pInTexels + (y * InStride);
				for (int x = 0; x < pInImage->SizeX; ++x)
				{
					FVector Normal = FVector(pInTexelsSwap[2] / 255.0f * 2.0f - 1.0f, pInTexelsSwap[1] / 255.0f * 2.0f - 1.0f, pInTexelsSwap[0] / 255.0f * 2.0f - 1.0f);
					Normal = Normal.GetSafeNormal();
					pInTexelsSwap[0] = 0;
					pInTexelsSwap[1] = static_cast<uint8>(FMath::RoundToInt((Normal.Y * 0.5f + 0.5f) * 255.f));
					pInTexelsSwap[2] = 0;
					pInTexelsSwap[3] = static_cast<uint8>(FMath::RoundToInt((Normal.X * 0.5f + 0.5f) * 255.f));

					pInTexelsSwap += 4;
				}
			}
		}
		else
		{
			// new preferred path: do not normalize normals here, use bNormalizeNormals in TextureCompressor
			for (int y = yStart; y < yEnd; ++y)
			{
				uint8* pInTexelsSwap = pInTexels + (y * InStride);
				for (int x = 0; x < pInImage->SizeX; ++x)
				{
					pInTexelsSwap[0] = 0;
					//pInTexelsSwap[1] = pInTexelsSwap[1]; // G
					pInTexelsSwap[3] = pInTexelsSwap[2]; // R->A
					pInTexelsSwap[2] = 0;

					pInTexelsSwap += 4;
				}
			}
		}
	}
	else if (pEncSettings->TextureFormatName == GTextureFormatNameASTC_NormalRG)
	{
		// input BGRA -> output RG,0,255

		if ( pEncSettings->bISPCTexcompNormalizeNormals )
		{
			// legacy behavior:
			// Re-normalize normals before dropping components
			// this is right, but doing it here makes this TextureFormat inconsistent with all others
			for (int y = yStart; y < yEnd; ++y)
			{
				uint8* pInTexelsSwap = pInTexels + (y * InStride);
				for (int x = 0; x < pInImage->SizeX; ++x)
				{
					FVector Normal = FVector(pInTexelsSwap[2] / 255.0f * 2.0f - 1.0f, pInTexelsSwap[1] / 255.0f * 2.0f - 1.0f, pInTexelsSwap[0] / 255.0f * 2.0f - 1.0f);
					Normal = Normal.GetSafeNormal();
					pInTexelsSwap[0] = static_cast<uint8>(FMath::RoundToInt((Normal.X * 0.5f + 0.5f) * 255.f));
					pInTexelsSwap[1] = static_cast<uint8>(FMath::RoundToInt((Normal.Y * 0.5f + 0.5f) * 255.f));
					pInTexelsSwap[2] = 0;
					pInTexelsSwap[3] = 255;

					pInTexelsSwap += 4;
				}
			}
		}
		else
		{
			// new preferred path: do not normalize normals here, use bNormalizeNormals in TextureCompressor
			for (int y = yStart; y < yEnd; ++y)
			{
				uint8* pInTexelsSwap = pInTexels + (y * InStride);
				for (int x = 0; x < pInImage->SizeX; ++x)
				{
					pInTexelsSwap[0] = pInTexelsSwap[2]; // R
					//pInTexelsSwap[1] = pInTexelsSwap[1]; // G
					pInTexelsSwap[2] = 0;
					pInTexelsSwap[3] = 255;

					pInTexelsSwap += 4;
				}
			}
		}
	}
	else
	{
		checkNoEntry();
	}

	rgba_surface insurface;
	insurface.ptr		= pInTexels + (yStart * InStride);
	insurface.width		= pInImage->SizeX;
	insurface.height	= yEnd - yStart;
	insurface.stride	= pInImage->SizeX * 4;
	
	check((yStart % pEncSettings->block_height) == 0);
	pOutTexels += (yStart / pEncSettings->block_height) * OutStride;
	CompressBlocksASTC(&insurface, pOutTexels, pEncSettings);
}

/**
 * Intel BC texture format handler.
 */
class FTextureFormatIntelISPCTexComp : public ITextureFormat
{
public:

	void* mDllHandle = nullptr;

	FTextureFormatIntelISPCTexComp()
	{
	}
	virtual ~FTextureFormatIntelISPCTexComp()
	{
		if ( mDllHandle != nullptr )
		{
			FPlatformProcess::FreeDllHandle(mDllHandle);
			mDllHandle = nullptr;
		}
	}

	void LoadDLLOnce()
	{
		// LoadDLL is delayed to first use
		// should only be called once
		check( mDllHandle == nullptr );

		FString DLLPath;
#if PLATFORM_WINDOWS
	#if  PLATFORM_CPU_ARM_FAMILY && !PLATFORM_WINDOWS_ARM64EC
		DLLPath = FPaths::EngineDir() / TEXT("Binaries/ThirdParty/Intel/ISPCTexComp/WinArm64-Release/ispc_texcomp.dll");
	#else
		DLLPath = FPaths::EngineDir() / TEXT("Binaries/ThirdParty/Intel/ISPCTexComp/Win64-Release/ispc_texcomp.dll");
	#endif
#elif PLATFORM_MAC
		DLLPath = FPaths::EngineDir() / TEXT("Binaries/ThirdParty/Intel/ISPCTexComp/Mac64-Release/libispc_texcomp.dylib");
#elif PLATFORM_LINUX
		DLLPath = FPaths::EngineDir() / TEXT("Binaries/ThirdParty/Intel/ISPCTexComp/Linux64-Release/libispc_texcomp.so");
#endif

		if (DLLPath.Len() > 0)
		{
			mDllHandle = FPlatformProcess::GetDllHandle(*DLLPath);
			UE_CLOGF(mDllHandle == nullptr, LogTextureFormatIntelISPCTexComp, Error, "Unable to load %ls", *DLLPath);
		}
		else
		{
			UE_LOGF(LogTextureFormatIntelISPCTexComp, Error, "Platform does not have an ispc_texcomp DLL/library");
		}
	}


	virtual bool AllowParallelBuild() const override
	{
		return true;
	}
	virtual FName GetEncoderName(FName Format) const override
	{
		static const FName IntelName("IntelISPC");
		return IntelName;
	}

	virtual FCbObject ExportGlobalFormatConfig(const FTextureBuildSettings& BuildSettings) const override
	{
		FCbWriter Writer;
		Writer.BeginObject("TextureFormatIntelISPCTexCompSettings");
		Writer.AddInteger("DefaultASTCQualityBySize", GetDefaultCompressionBySizeValue(FCbObjectView()));
		Writer.EndObject();
		return Writer.Save().AsObject();
	}

	// Return the version for the DX11 formats BC6H and BC7 (not ASTC)
	virtual uint16 GetVersion(
		FName Format,
		const FTextureBuildSettings* BuildSettings = nullptr
	) const override
	{
		return BASE_ISPC_DX11_FORMAT_VERSION;
	}
	
	virtual FString GetDerivedDataKeyString(const FTextureBuildSettings& InBuildSettings, int32 InMipCount, const FIntVector3& InMip0Dimensions) const override
	{
		// ASTC block size chosen is in PixelFormat
		EPixelFormat PixelFormat = GetPixelFormatForBuildSettings(InBuildSettings);
		
 		return FString::Printf(TEXT("ISPC_%d"), (int)PixelFormat);
	}

	virtual void GetSupportedFormats(TArray<FName>& OutFormats) const override
	{
		OutFormats.Append(GSupportedTextureFormatNames, UE_ARRAY_COUNT(GSupportedTextureFormatNames));
	}

	static void SetupScans(const FImage& InImage, int BlockWidth, int BlockHeight, FCompressedImage2D& OutCompressedImage, FMultithreadSettings &MultithreadSettings)
	{
		const int AlignedSizeX = AlignArbitrary(InImage.SizeX, BlockWidth);
		const int AlignedSizeY = AlignArbitrary(InImage.SizeY, BlockHeight);
		const int WidthInBlocks = AlignedSizeX / BlockWidth;
		const int HeightInBlocks = AlignedSizeY / BlockHeight;
		const int64 SizePerSlice = (int64)WidthInBlocks * HeightInBlocks * BLOCK_SIZE_IN_BYTES;
		// ISPC pads sizes, but OutCompressedImage gets the unpadded sizes :
		OutCompressedImage.RawData.AddUninitialized(SizePerSlice * InImage.NumSlices);
		OutCompressedImage.SizeX = InImage.SizeX;
		OutCompressedImage.SizeY = InImage.SizeY;

		// When we allow async tasks to execute we do so with BlockHeight lines of the image per task
		// This isn't optimal for long thin textures, but works well with how ISPC works
		MultithreadSettings.iScansPerTask = BlockHeight;
		MultithreadSettings.iNumTasks = FMath::Max((AlignedSizeY / MultithreadSettings.iScansPerTask) - 1, 0);
	}

	static void PadImageToBlockSize(FImage &InOutImage, int BlockWidth, int BlockHeight, int BytesPerPixel)
	{
		const int AlignedSizeX = AlignArbitrary(InOutImage.SizeX, BlockWidth);
		const int AlignedSizeY = AlignArbitrary(InOutImage.SizeY, BlockHeight);
		const int64 AlignedSliceSize = (int64)AlignedSizeX * AlignedSizeY * BytesPerPixel;
		const int64 AlignedTotalSize = AlignedSliceSize * InOutImage.NumSlices;
		const int64 OriginalSliceSize = (int64)InOutImage.SizeX * InOutImage.SizeY * BytesPerPixel;

		// Early out if no padding is necessary
		if (AlignedSizeX == InOutImage.SizeX && AlignedSizeY == InOutImage.SizeY)
		{
			return;
		}
		
		// ISPCtexcomp crashes on images not of block-aligned size

		// Allocate temp buffer
		TArray64<uint8> TempBuffer;
		TempBuffer.SetNumUninitialized(AlignedTotalSize);

		const int PaddingX = AlignedSizeX - InOutImage.SizeX;
		const int PaddingY = AlignedSizeY - InOutImage.SizeY;
		const int SrcStride = InOutImage.SizeX * BytesPerPixel;
		const int DstStride = AlignedSizeX * BytesPerPixel;

		for (int SliceIndex = 0; SliceIndex < InOutImage.NumSlices; ++SliceIndex)
		{
			uint8* DstData = ((uint8*)TempBuffer.GetData()) + SliceIndex * AlignedSliceSize;
			const uint8* SrcData = ((uint8*)InOutImage.RawData.GetData()) + SliceIndex * OriginalSliceSize;

			// Copy all of SrcData and pad on X-axis:
			for (int Y = 0; Y < InOutImage.SizeY; ++Y)
			{
				FMemory::Memcpy(DstData, SrcData, SrcStride);
				SrcData += SrcStride - BytesPerPixel;	// Src: Last pixel on this row
				DstData += SrcStride;					// Dst: Beginning of the padded region at the end of this row
				for (int PadX = 0; PadX < PaddingX; PadX++)
				{
					// Replicate right-most pixel as padding on X-axis
					FMemory::Memcpy(DstData, SrcData, BytesPerPixel);
					DstData += BytesPerPixel;
				}
				SrcData += BytesPerPixel;				// Src & Dst: Beginning of next row
			}

			// Replicate last row as padding on Y-axis:
			SrcData = DstData - DstStride;				// Src: Beginning of the last row (of DstData)
			for (int PadY = 0; PadY < PaddingY; PadY++)
			{
				FMemory::Memcpy(DstData, SrcData, DstStride);
				DstData += DstStride;					// Dst: Beginning of the padded region at the end of this row
			}
		}

		// Replace InOutImage with the new data
		InOutImage.RawData = MoveTemp(TempBuffer);
		InOutImage.SizeX = AlignedSizeX;
		InOutImage.SizeY = AlignedSizeY;
	}

	virtual EPixelFormat GetEncodedPixelFormat(const FTextureBuildSettings& InBuildSettings, bool bInImageHasAlphaChannel) const override
	{
		return GetPixelFormatForBuildSettings(InBuildSettings);
	}

	EPixelFormat GetPixelFormatForBuildSettings(const FTextureBuildSettings& BuildSettings) const
	{
		if (BuildSettings.TextureFormatName == GTextureFormatNameBC6H)
		{
			return PF_BC6H;
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameBC7)
		{
			return PF_BC7;
		}
		else
		{
			int _Width, _Height;

			return GetASTCQualityFormat(_Width, _Height, BuildSettings);
		}
	}

	virtual bool CompressImage(
		const FImage& InImage,
		const FTextureBuildSettings& BuildSettings,
		const FIntVector3& InMip0Dimensions,
		int32 InMip0NumSlicesNoDepth,
		int32 InMipIndex,
		int32 InMipCount,
		FStringView DebugTexturePathName,
		bool bImageHasAlphaChannel,
		FCompressedImage2D& OutCompressedImage
		) const override
	{
		check(InImage.SizeX > 0);
		check(InImage.SizeY > 0);
		check(InImage.NumSlices > 0);

		// Load DLL on first use :
		UE_CALL_ONCE( const_cast<FTextureFormatIntelISPCTexComp *>(this)->LoadDLLOnce );

		bool bCompressionSucceeded = false;

		int BlockWidth = 0;
		int BlockHeight = 0;

		const bool bUseTasks = true;
		FMultithreadSettings MultithreadSettings;

		EPixelFormat CompressedPixelFormat = GetEncodedPixelFormat(BuildSettings, bImageHasAlphaChannel);

		if ( BuildSettings.TextureFormatName == GTextureFormatNameBC6H )
		{
			FImage Image;
			InImage.CopyTo(Image, ERawImageFormat::RGBA16F, EGammaSpace::Linear);

			FImageCore::SanitizeFloat16AndSetAlphaOpaqueForBC6H(Image);

			bc6h_enc_settings settings;
			GetProfile_bc6h_basic(&settings);

			SetupScans(Image, 4, 4, OutCompressedImage, MultithreadSettings);
			PadImageToBlockSize(Image, 4, 4, 4*2);
			FMultithreadedCompression<bc6h_enc_settings>::Compress(MultithreadSettings, settings, Image, OutCompressedImage, &IntelBC6HCompressScans, bUseTasks);

			bCompressionSucceeded = true;
		}
		else if ( BuildSettings.TextureFormatName == GTextureFormatNameBC7 )
		{
			FImage Image;
			InImage.CopyTo(Image, ERawImageFormat::BGRA8, BuildSettings.GetDestGammaSpace());

			bc7_enc_settings settings;
			if ( bImageHasAlphaChannel )
			{
				GetProfile_alpha_basic(&settings);
			}
			else
			{
				GetProfile_basic(&settings);
			}

			SetupScans(Image, 4, 4, OutCompressedImage, MultithreadSettings);
			PadImageToBlockSize(Image, 4, 4, 4*1);
			FMultithreadedCompression<bc7_enc_settings>::Compress(MultithreadSettings, settings, Image, OutCompressedImage, &IntelBC7CompressScans, bUseTasks);

			bCompressionSucceeded = true;
		}
		else
		{
			bool bIsNormalMap = (BuildSettings.TextureFormatName == GTextureFormatNameASTC_NormalAG ||
				BuildSettings.TextureFormatName == GTextureFormatNameASTC_NormalRG);
				
			bool bHDRImage = BuildSettings.TextureFormatName == GTextureFormatNameASTC_RGB_HDR;
			check( ! bHDRImage ); // HDR not supported here
			
			GetASTCQualityFormat( BlockWidth, BlockHeight, BuildSettings );

			FASTCEncoderSettings EncoderSettings;
			EncoderSettings.TextureFormatName = BuildSettings.TextureFormatName;
			
			if ( bIsNormalMap )
			{
				check( BlockWidth == 6 && BlockHeight == 6 );

				if (BuildSettings.TextureFormatName == GTextureFormatNameASTC_NormalAG)
				{
					GetProfile_astc_alpha_fast(&EncoderSettings, BlockWidth, BlockHeight);
				}
				else if (BuildSettings.TextureFormatName == GTextureFormatNameASTC_NormalRG)
				{
					GetProfile_astc_fast(&EncoderSettings, BlockWidth, BlockHeight);
				}
				else
				{	
					checkNoEntry();
				}

				if ( BuildSettings.bNormalizeNormals || BuildSettings.bUseNewMipFilter )
				{
					// newer texture build, do NOT normalize normals here
					// it is done by bNormalizeNormals if wanted, at the TextureCompressor level before getting to the TextureFormat
					EncoderSettings.bISPCTexcompNormalizeNormals = false;
				}
				else
				{
					// old texture build
					// enable legacy behavior to maintain same output
					// we would prefer bISPCTexcompNormalizeNormals to just always be off and use bNormalizeNormals instead
					EncoderSettings.bISPCTexcompNormalizeNormals = true;
				}
			}
			else
			{			
				if( BuildSettings.TextureFormatName == GTextureFormatNameASTC_RGB || !bImageHasAlphaChannel )
				{
					// even if Name was RGBA we still use the RGB profile if !bImageHasAlphaChannel
					//	so that "Compress Without Alpha" can force us to opaque

					EncoderSettings.TextureFormatName = GTextureFormatNameASTC_RGB; // <- this will force A to 255 in IntelASTCCompressScans

					GetProfile_astc_fast(&EncoderSettings, BlockWidth, BlockHeight);
				}
				else
				{
					EncoderSettings.TextureFormatName = GTextureFormatNameASTC_RGBA;
					
					GetProfile_astc_alpha_fast(&EncoderSettings, BlockWidth, BlockHeight);
				}
			}
			
			{
				check(EncoderSettings.block_width!=0);

				FImage Image;
				InImage.CopyTo(Image, ERawImageFormat::BGRA8, BuildSettings.GetDestGammaSpace());

				SetupScans(Image, EncoderSettings.block_width, EncoderSettings.block_height, OutCompressedImage, MultithreadSettings);
				PadImageToBlockSize(Image, EncoderSettings.block_width, EncoderSettings.block_height, 4 * 1);
				
#if DEBUG_SAVE_INTERMEDIATE_IMAGES
				// To DebugDump specific files, modify DebugDumpFilter here
				//  (currently not exposed to command line, but could be)
				//static FString DebugDumpFilter(TEXT("*DummySpriteTexture*"));;
				static FString DebugDumpFilter(TEXT("*"));
				//static FString DebugDumpFilter;

				bool SaveInputOutput = false;
				if ( ! DebugDumpFilter.IsEmpty() )
				{
					SaveInputOutput = FWildcardString::IsMatchSubstring(*DebugDumpFilter, DebugTexturePathName.GetData(), DebugTexturePathName.GetData() + DebugTexturePathName.Len(), ESearchCase::IgnoreCase);
				}

				FString DebugDumpFileNameBase;
				if (SaveInputOutput)
				{
					FString FileName = FString::Printf(TEXT("%.*s_%dx%d_%s_%dx%d"), DebugTexturePathName.Len(), DebugTexturePathName.GetData(), 
						Image.SizeX, Image.SizeY, *BuildSettings.TextureFormatName.ToString(), BlockWidth,BlockHeight);

					// Object paths a) can contain slashes as its a path, and we dont want a hierarchy and b) can have random characters we don't want
					FileName = FPaths::MakeValidFileName(FileName, TEXT('_'));

					FileName = FPaths::ProjectSavedDir() + TEXT("ISPCDebugImages/") + FileName;
					
					// limit to _MAX_PATH
					if ( FileName.Len() >= 256 )
					{
						FileName = FileName.Right(255);
					}

					DebugDumpFileNameBase = FileName;
				}

				if (SaveInputOutput)
				{
					const FString FileName = FString::Printf(TEXT("%s-in.bmp"), *DebugDumpFileNameBase );
					FArchive* FileWriter = IFileManager::Get().CreateFileWriter(*FileName);
					SaveImageAsBMP(*FileWriter, Image.RawData.GetData(), 4, Image.SizeX, Image.SizeY);
					delete FileWriter;
				}
#endif

				FMultithreadedCompression<FASTCEncoderSettings>::Compress(MultithreadSettings, EncoderSettings, Image, OutCompressedImage, &IntelASTCCompressScans, bUseTasks);

				bCompressionSucceeded = true;
				
#if DEBUG_SAVE_INTERMEDIATE_IMAGES
				// (save swizzled/fixed-up input as BMP):
				if (SaveInputOutput)
				{
					const FString FileName = FString::Printf(TEXT("%s-in-swiz.bmp"), *DebugDumpFileNameBase );
					FArchive* FileWriter = IFileManager::Get().CreateFileWriter(*FileName);
					SaveImageAsBMP(*FileWriter, Image.RawData.GetData(), 4, Image.SizeX, Image.SizeY);
					delete FileWriter;
				}

				// (save output as .astc file):
				if (SaveInputOutput)
				{
					const FString FileName = FString::Printf(TEXT("%s-out.astc"), *DebugDumpFileNameBase );
					FArchive* FileWriter = IFileManager::Get().CreateFileWriter(*FileName);
					SaveImageAsASTC(*FileWriter, OutCompressedImage.RawData.GetData(), OutCompressedImage.SizeX, OutCompressedImage.SizeY, EncoderSettings.block_width, EncoderSettings.block_height);
					delete FileWriter;
				}
#endif
			}
		}
		OutCompressedImage.PixelFormat = CompressedPixelFormat;
		OutCompressedImage.SizeX = InImage.SizeX;
		OutCompressedImage.SizeY = InImage.SizeY;
		OutCompressedImage.NumSlicesWithDepth = InImage.NumSlices;
		return bCompressionSucceeded;
	}
};

/**
 * Module for DXT texture compression.
 */
static ITextureFormat* Singleton = nullptr;

class FTextureFormatIntelISPCTexCompModule : public ITextureFormatModule
{
public:
	FTextureFormatIntelISPCTexCompModule()
	{
	}

	virtual ~FTextureFormatIntelISPCTexCompModule()
	{
		if ( Singleton )
		{
			delete Singleton;
			Singleton = nullptr;
		}
	}
	
	virtual bool CanCallGetTextureFormats() override { return false; }

	virtual ITextureFormat* GetTextureFormat()
	{
		if (!Singleton)
		{
			Singleton = new FTextureFormatIntelISPCTexComp();
		}
		return Singleton;
	}

	static inline UE::DerivedData::TBuildFunctionFactory<FIntelISPCTexCompTextureBuildFunction> BuildFunctionFactory;
};

IMPLEMENT_MODULE(FTextureFormatIntelISPCTexCompModule, TextureFormatIntelISPCTexComp);
