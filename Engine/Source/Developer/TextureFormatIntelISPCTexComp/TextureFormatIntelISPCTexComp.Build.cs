// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TextureFormatIntelISPCTexComp : ModuleRules
{
	public TextureFormatIntelISPCTexComp(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(new string[]
		{
			"DerivedDataCache",
			"TextureCompressor",
			"TextureFormat",
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"ImageCore",
			"TextureBuild",
		});

		AddEngineThirdPartyPrivateStaticDependencies(Target, "IntelISPCTexComp");

		if (Target.Platform != UnrealTargetPlatform.Win64 &&
			Target.Platform != UnrealTargetPlatform.Mac &&
			Target.Platform != UnrealTargetPlatform.Linux)
		{
			PrecompileForTargets = PrecompileTargetsType.None;
		}
	}
}
